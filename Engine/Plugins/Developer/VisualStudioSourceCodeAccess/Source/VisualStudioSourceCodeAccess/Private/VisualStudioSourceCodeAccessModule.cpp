// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioSourceCodeAccessModule.h"
#include "VisualStudioSourceCodeAccessSettings.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "ISettingsModule.h"

IMPLEMENT_MODULE( FVisualStudioSourceCodeAccessModule, VisualStudioSourceCodeAccess );

#define LOCTEXT_NAMESPACE "VisualStudioSourceCodeAccessor"

LLM_DEFINE_TAG(VisualStudioSourceCodeAccess);

FVisualStudioSourceCodeAccessModule::FVisualStudioSourceCodeAccessModule()
	: VisualStudioSourceCodeAccessor(MakeShareable(new FVisualStudioSourceCodeAccessor()))
{
}

void FVisualStudioSourceCodeAccessModule::StartupModule()
{
	LLM_SCOPE_BYTAG(VisualStudioSourceCodeAccess);

	VisualStudioSourceCodeAccessor->Startup();

	// Add all the explicit version wrappers. If one of these is selected, UBT will generate project files in the appropriate format. Editor behavior is still to detect which version to use
	// from the solution on disk.
	RegisterWrapper("VisualStudio2022", LOCTEXT("VisualStudio2022", "Visual Studio 2022"), LOCTEXT("UsingVisualStudio2022", "Open source code files in Visual Studio 2022"));
	RegisterWrapper("VisualStudio2026", LOCTEXT("VisualStudio2026", "Visual Studio 2026"), LOCTEXT("UsingVisualStudio2026", "Open source code files in Visual Studio 2026"));

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &VisualStudioSourceCodeAccessor.Get() );

	// Register to display our settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "Visual Studio",
			LOCTEXT("TargetSettingsName", "Visual Studio"),
			LOCTEXT("TargetSettingsDescription", "Control how Visual Studio accesses source code, when set as the source code accessor."),
			GetMutableDefault<UVisualStudioSourceCodeAccessSettings>()
		);
	}
}

void FVisualStudioSourceCodeAccessModule::ShutdownModule()
{
	// unbind all the explicit version wrappers
	for (TSharedRef<FVisualStudioSourceCodeAccessorWrapper> &Wrapper : Wrappers)
	{
		IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &Wrapper.Get());
	}

	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &VisualStudioSourceCodeAccessor.Get());

	VisualStudioSourceCodeAccessor->Shutdown();
}

FVisualStudioSourceCodeAccessor& FVisualStudioSourceCodeAccessModule::GetAccessor()
{
	return VisualStudioSourceCodeAccessor.Get();
}

void FVisualStudioSourceCodeAccessModule::RegisterWrapper(FName Name, FText NameText, FText DescriptionText)
{
	TSharedRef<FVisualStudioSourceCodeAccessorWrapper> Wrapper = MakeShareable(new FVisualStudioSourceCodeAccessorWrapper(Name, NameText, DescriptionText, VisualStudioSourceCodeAccessor));
	Wrappers.Add(Wrapper);
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &Wrapper.Get());
}

#undef LOCTEXT_NAMESPACE