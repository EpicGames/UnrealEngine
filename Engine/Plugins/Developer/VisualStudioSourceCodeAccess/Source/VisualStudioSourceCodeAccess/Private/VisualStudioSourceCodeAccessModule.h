// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "VisualStudioSourceCodeAccessor.h"
#include "VisualStudioSourceCodeAccessorWrapper.h"

class FVisualStudioSourceCodeAccessModule : public IModuleInterface
{
public:
	FVisualStudioSourceCodeAccessModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FVisualStudioSourceCodeAccessor& GetAccessor();

private:
	void RegisterWrapper(FName Name, FText NameText, FText DescriptionText);

	TSharedRef<FVisualStudioSourceCodeAccessor> VisualStudioSourceCodeAccessor;
	TArray<TSharedRef<FVisualStudioSourceCodeAccessorWrapper>> Wrappers;
};