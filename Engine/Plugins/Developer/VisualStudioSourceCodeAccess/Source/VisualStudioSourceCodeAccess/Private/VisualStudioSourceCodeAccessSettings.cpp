// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioSourceCodeAccessSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualStudioSourceCodeAccessSettings)

UVisualStudioSourceCodeAccessSettings::UVisualStudioSourceCodeAccessSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
