// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VisualStudioSourceCodeAccessSettings.generated.h"

UCLASS(config=EditorSettings)
class UVisualStudioSourceCodeAccessSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Open a .uproject directly in Visual Studio instead of the generated .sln solution. */
	UPROPERTY(Config, EditAnywhere, Category = "Visual Studio Source Code", meta = (DisplayName = "Direct Unreal Project Support"))
	bool bUproject;

	/** Prefer preview releases of Visual Studio. */
	UPROPERTY(Config, EditAnywhere, Category = "Visual Studio Source Code", meta = (DisplayName = "Prefer Preview Releases"))
	bool bPreview;
};
