// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioSourceCodeAccessor.h"
#include "VisualStudioSourceCodeAccessModule.h"
#include "VisualStudioSourceCodeAccessSettings.h"
#include "ISourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"
#include "HAL/PlatformTime.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/FileHelper.h"
#include "Misc/MonitoredProcess.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Algo/Transform.h"
//#include "GameProjectGenerationModule.h"

#if WITH_EDITOR
#include "IHotReload.h"
#endif

#include "VisualStudioDTE.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"
	#include <unknwn.h>
	#include "Microsoft/COMPointer.h"
	#include "Setup.Configuration.h"
	#include <tlhelp32.h>
	#include <wbemidl.h>
	#pragma comment(lib, "wbemuuid.lib")
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogVSAccessor, Log, All);

#define LOCTEXT_NAMESPACE "VisualStudioSourceCodeAccessor"

// Wrapper class for COM BSTRs, similar to CComBSTR from ATL
class FComBSTR : FNoncopyable
{
public:
	FComBSTR(const FString& Other)
	{
		String = SysAllocString(*Other);
	}

	~FComBSTR()
	{
		SysFreeString(String);
	}

	operator BSTR()
	{
		return String;
	}

private:
	BSTR String;
};

class FCoInitializeScope
{
public:
	FCoInitializeScope()
	{
		bInitialized = FWindowsPlatformMisc::CoInitialize();
		UE_CLOGF(!bInitialized, LogVSAccessor, Error, "ERROR - Could not initialize COM library!");
	}

	~FCoInitializeScope()
	{
		if (bInitialized)
		{
			FWindowsPlatformMisc::CoUninitialize();
		}
	}

	bool IsValid() const
	{
		return bInitialized;
	}

private:

	bool bInitialized;
};

/** The VS query in progress message */
TWeakPtr<class SNotificationItem> VSNotificationPtr = NULL;

/** Return codes when trying to access an existing VS instance */
enum class EAccessVisualStudioResult : uint8
{
	/** An instance of Visual Studio is available, and the relevant output variables have been filled in */
	VSInstanceIsOpen,
	/** An instance of Visual Studio is not available */
	VSInstanceIsNotOpen,
	/** An instance of Visual Studio is open, but could not be fully queried because it is blocked by a modal operation - this may succeed later */
	VSInstanceIsBlocked,
	/** It is unknown whether an instance of Visual Studio is available, as an error occurred when performing the check */
	VSInstanceUnknown,
};

/** save all open documents in visual studio, when recompiling */
static void OnModuleCompileStarted(bool bIsAsyncCompile)
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.GetAccessor().SaveAllOpenDocuments();
}

int32 GetVisualStudioVersionForCompiler()
{
#if _MSC_VER >= 1950
	return 18; // Visual Studio 2026
#elif _MSC_VER >= 1930
	return 17; // Visual Studio 2022
#elif defined(__clang__)
	return 0;
#else
	#error "FVisualStudioSourceCodeAccessor::RefreshAvailability - Unknown _MSC_VER! Please update this code for this version of MSVC."
	return 0;
#endif //_MSVC_VER
}

int32 GetVisualStudioVersionForSolution(const FString& InSolutionFile)
{
	if (FPaths::DirectoryExists(InSolutionFile))
	{
		// .uproject support uses a directory root instead of a file, and has no solution to check so defer to the version based on the compiler
		return 0;
	}
	static const FString VisualStudioVersionString = TEXT("# Visual Studio ");

	FString SolutionFileContents;
	if (FFileHelper::LoadFileToString(SolutionFileContents, *InSolutionFile))
	{
		// Find the format version from the file, it will look like "# Visual Studio 14"
		const int32 VersionStringStart = SolutionFileContents.Find(VisualStudioVersionString);
		if (VersionStringStart != INDEX_NONE)
		{
			const TCHAR* VersionChar = *SolutionFileContents + VersionStringStart + VisualStudioVersionString.Len();

			const TCHAR VersionSuffix[] = TEXT("Version ");
			if (FCString::Strnicmp(VersionChar, VersionSuffix, UE_ARRAY_COUNT(VersionSuffix) - 1) == 0)
			{
				VersionChar += UE_ARRAY_COUNT(VersionSuffix) - 1;
			}

			FString VersionString;
			for (; FChar::IsDigit(*VersionChar); ++VersionChar)
			{
				VersionString.AppendChar(*VersionChar);
			}

			int32 Version = 0;
			LexFromString(Version, *VersionString);
			return Version;
		}
	}

	return 0;
}

void FVisualStudioSourceCodeAccessor::Startup()
{
	VSLaunchTime = 0.0;

#if WITH_EDITOR
	// Setup compilation for saving all VS documents upon compilation start
	SaveVisualStudioDocumentsDelegateHandle = IHotReloadModule::Get().OnModuleCompilerStarted().AddStatic( &OnModuleCompileStarted );
#endif

	// Cache this so we don't have to do it on a background thread
	GetSolutionPath();

	RefreshAvailability();
}

void FVisualStudioSourceCodeAccessor::RefreshAvailability()
{
	Locations.Reset();

	// Minor optimization, as each call to AddVisualStudioVersionUsingVisualStudioSetupAPI will make it's
	// own calls to ::CoInitialize/::CoUninitialize. If we do our own calls here then they will just inc/dec
	// the internal ref count rather than potentially creating and destroying resources for each call.
	FCoInitializeScope CoInitialze;
	if (CoInitialze.IsValid())
	{
		AddVisualStudioVersionUsingVisualStudioSetupAPI(MinimumVisualStudioVersion);
	}
}

void FVisualStudioSourceCodeAccessor::Shutdown()
{
#if WITH_EDITOR
	// Unregister the hot-reload callback
	if(IHotReloadModule::IsAvailable())
	{
		IHotReloadModule::Get().OnModuleCompilerStarted().Remove( SaveVisualStudioDocumentsDelegateHandle );
	}
#endif
}

#if WITH_VISUALSTUDIO_DTE

bool IsVisualStudioDTEMoniker(const FString& InName, const TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation>& InLocations)
{
	for(int32 VSVersion = 0; VSVersion < InLocations.Num(); VSVersion++)
	{
		if(InName.StartsWith(InLocations[VSVersion].ROTMoniker))
		{
			return true;
		}
	}

	return false;
}

/** Query all opened instances of Visual Studio to retrieve their respective opened solutions. */
TArray<FString> RetrieveOpenedVisualStudioSolutionsViaDTE(const TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation>& InLocations)
{
	TArray<FString> OpenedSolutions;
	// Open the Running Object Table (ROT)
	IRunningObjectTable* RunningObjectTable;
	if (SUCCEEDED(GetRunningObjectTable(0, &RunningObjectTable)) && RunningObjectTable)
	{
		IEnumMoniker* MonikersTable;
		if (SUCCEEDED(RunningObjectTable->EnumRunning(&MonikersTable)))
		{
			MonikersTable->Reset();

			// Look for all visual studio instances in the ROT
			IMoniker* CurrentMoniker;
			while (MonikersTable->Next(1, &CurrentMoniker, NULL) == S_OK)
			{
				IBindCtx* BindContext;
				LPOLESTR OutName;
				if (SUCCEEDED(CreateBindCtx(0, &BindContext)) && SUCCEEDED(CurrentMoniker->GetDisplayName(BindContext, NULL, &OutName)))
				{
					if (IsVisualStudioDTEMoniker(FString(OutName), InLocations))
					{
						TComPtr<IUnknown> ComObject;
						if (SUCCEEDED(RunningObjectTable->GetObject(CurrentMoniker, &ComObject)))
						{
							TComPtr<EnvDTE::_DTE> TempDTE;
							if (SUCCEEDED(TempDTE.FromQueryInterface(__uuidof(EnvDTE::_DTE), ComObject)))
							{
								// Get the solution path for this instance
								TComPtr<EnvDTE::_Solution> Solution;
								BSTR OutPath = nullptr;
								if (SUCCEEDED(TempDTE->get_Solution(&Solution)) &&
									SUCCEEDED(Solution->get_FullName(&OutPath)))
								{
									FString Filename(OutPath);
									FPaths::NormalizeFilename(Filename);

									OpenedSolutions.AddUnique(Filename);

									SysFreeString(OutPath);
								}
								else
								{
									UE_LOGF(LogVSAccessor, Log, "Visual Studio is open but could not be queried - it may still be initializing or blocked by a modal operation");
								}
							}
							else
							{
								UE_LOGF(LogVSAccessor, Warning, "Could not get DTE interface from returned Visual Studio instance");
							}
						}
						else
						{
							UE_LOGF(LogVSAccessor, Warning, "Couldn't get Visual Studio COM object");
						}
					}
				}
				else
				{
					UE_LOGF(LogVSAccessor, Warning, "Couldn't get display name");
				}
				BindContext->Release();
				CurrentMoniker->Release();
			}
			MonikersTable->Release();
		}
		else
		{
			UE_LOGF(LogVSAccessor, Warning, "Couldn't enumerate ROT table");
		}
		RunningObjectTable->Release();
	}
	else
	{
		UE_LOGF(LogVSAccessor, Warning, "Couldn't get ROT table");
	}

	return OpenedSolutions;
}

/** Accesses the correct visual studio instance if possible. */
EAccessVisualStudioResult AccessVisualStudioViaDTE(TComPtr<EnvDTE::_DTE>& OutDTE, const FString& InSolutionPath, const TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation>& InLocations)
{
	EAccessVisualStudioResult AccessResult = EAccessVisualStudioResult::VSInstanceIsNotOpen;

	// Open the Running Object Table (ROT)
	IRunningObjectTable* RunningObjectTable;
	if(SUCCEEDED(GetRunningObjectTable(0, &RunningObjectTable)) && RunningObjectTable)
	{
		IEnumMoniker* MonikersTable;
		if(SUCCEEDED(RunningObjectTable->EnumRunning(&MonikersTable)))
		{
			MonikersTable->Reset();

			// Look for all visual studio instances in the ROT
			IMoniker* CurrentMoniker;
			while(AccessResult != EAccessVisualStudioResult::VSInstanceIsOpen && MonikersTable->Next(1, &CurrentMoniker, NULL) == S_OK)
			{
				IBindCtx* BindContext;
				LPOLESTR OutName;
				if(SUCCEEDED(CreateBindCtx(0, &BindContext)) && SUCCEEDED(CurrentMoniker->GetDisplayName(BindContext, NULL, &OutName)))
				{
					if(IsVisualStudioDTEMoniker(FString(OutName), InLocations))
					{
						TComPtr<IUnknown> ComObject;
						if(SUCCEEDED(RunningObjectTable->GetObject(CurrentMoniker, &ComObject)))
						{
							TComPtr<EnvDTE::_DTE> TempDTE;
							if (SUCCEEDED(TempDTE.FromQueryInterface(__uuidof(EnvDTE::_DTE), ComObject)))
							{
								// Get the solution path for this instance
								// If it equals the solution we would have opened above in RunVisualStudio(), we'll take that
								TComPtr<EnvDTE::_Solution> Solution;
								BSTR OutPath = nullptr;
								if (SUCCEEDED(TempDTE->get_Solution(&Solution)) &&
									SUCCEEDED(Solution->get_FullName(&OutPath)))
								{
									FString Filename(OutPath);
									FPaths::NormalizeFilename(Filename);

									if (Filename == InSolutionPath || InSolutionPath.IsEmpty())
									{
										OutDTE = TempDTE;
										AccessResult = EAccessVisualStudioResult::VSInstanceIsOpen;
									}

									SysFreeString(OutPath);
								}
								else
								{
									UE_LOGF(LogVSAccessor, Log, "Visual Studio is open but could not be queried - it may still be initializing or blocked by a modal operation");
									AccessResult = EAccessVisualStudioResult::VSInstanceIsBlocked;
								}
							}
							else
							{
								UE_LOGF(LogVSAccessor, Warning, "Could not get DTE interface from returned Visual Studio instance");
								AccessResult = EAccessVisualStudioResult::VSInstanceIsBlocked;
							}
						}
						else
						{
							UE_LOGF(LogVSAccessor, Warning, "Couldn't get Visual Studio COM object");
							AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
						}
					}
				}
				else
				{
					UE_LOGF(LogVSAccessor, Warning, "Couldn't get display name");
					AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
				}
				BindContext->Release();
				CurrentMoniker->Release();
			}
			MonikersTable->Release();
		}
		else
		{
			UE_LOGF(LogVSAccessor, Warning, "Couldn't enumerate ROT table");
			AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
		}
		RunningObjectTable->Release();
	}
	else
	{
		UE_LOGF(LogVSAccessor, Warning, "Couldn't get ROT table");
		AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
	}

	return AccessResult;
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioSolutionViaDTE()
{
	// Initialize the com library, if not already by this thread
	FCoInitializeScope CoInitialze;
	if (!CoInitialze.IsValid())
	{
		return false;
	}

	bool bSuccess = false;

	TComPtr<EnvDTE::_DTE> DTE;
	const FString SolutionPath = GetSolutionPath();
	switch (AccessVisualStudioViaDTE(DTE, SolutionPath, GetPrioritizedVisualStudioVersions(SolutionPath)))
	{
	case EAccessVisualStudioResult::VSInstanceIsOpen:
		{
			// Set Focus on Visual Studio
			TComPtr<EnvDTE::Window> MainWindow;
			if (SUCCEEDED(DTE->get_MainWindow(&MainWindow)) &&
				SUCCEEDED(MainWindow->Activate()))
			{
				bSuccess = true;
			}
			else
			{
				UE_LOGF(LogVSAccessor, Warning, "Couldn't set focus on Visual Studio.");
			}
		}
		break;

	case EAccessVisualStudioResult::VSInstanceIsNotOpen:
		{
			// Automatically fail if there's already an attempt in progress
			if (!IsVSLaunchInProgress())
			{
				bSuccess = RunVisualStudioAndOpenSolution(SolutionPath);
			}
		}
		break;

	default:
		// Do nothing if we failed the VS detection, otherwise we could get stuck in a loop of constantly 
		// trying to open a VS instance since we can't detect that one is already running
		break;
	}

	return bSuccess;
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioFilesInternalViaDTE(const TArray<FileOpenRequest>& Requests, bool& bWasDeferred)
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));

	FCoInitializeScope CoInitialze;
	if (!CoInitialze.IsValid())
	{
		UE_LOGF(LogVSAccessor, Error, "ERROR - Could not initialize COM library!");
		return false;
	}
	
	FString SolutionPath = GetSolutionPath();
	TArray<VisualStudioLocation> InstalledLocations = GetPrioritizedVisualStudioVersions(SolutionPath);

	// Even if a solution is specified, also check open solutions and make an educated guess about the best path based on the files being requested by finding the corresponding .sln/.slnf files: 
	TArray<FString> CurrentlyOpenedSolutions = RetrieveOpenedVisualStudioSolutionsViaDTE(InstalledLocations);
	if (!SolutionPath.IsEmpty())
	{
		CurrentlyOpenedSolutions.Add(SolutionPath);
	}
	SolutionPath = RetrieveSolutionForFileOpenRequests(Requests, CurrentlyOpenedSolutions);

	bool bDefer = false, bSuccess = false;
	TComPtr<EnvDTE::_DTE> DTE;
	
	switch (AccessVisualStudioViaDTE(DTE, SolutionPath, InstalledLocations))
	{
	case EAccessVisualStudioResult::VSInstanceIsOpen:
		{
			// Set Focus on Visual Studio
			TComPtr<EnvDTE::Window> MainWindow;
			if (SUCCEEDED(DTE->get_MainWindow(&MainWindow)) &&
				SUCCEEDED(MainWindow->Activate()))
			{
				// Get ItemOperations
				TComPtr<EnvDTE::ItemOperations> ItemOperations;
				if (SUCCEEDED(DTE->get_ItemOperations(&ItemOperations)))
				{
					for ( const FileOpenRequest& Request : Requests )
					{
						// Check that the file actually exists first
						if ( !FPaths::FileExists(Request.FullPath) )
						{
							SourceCodeAccessModule.OnOpenFileFailed().Broadcast(Request.FullPath);
							bSuccess |= false;
							continue;
						}

						// Open File
						FString PlatformFilename = Request.FullPath;
						FPaths::MakePlatformFilename(PlatformFilename);
						auto ANSIPath = StringCast<ANSICHAR>(*PlatformFilename);
						FComBSTR COMStrFileName(ANSIPath.Get());
						FComBSTR COMStrKind(EnvDTE::vsViewKindTextView);
						TComPtr<EnvDTE::Window> Window;
						if ( SUCCEEDED(ItemOperations->OpenFile(COMStrFileName, COMStrKind, &Window)) )
						{
							// If we've made it this far - we've opened the file.  it doesn't matter if
							// we successfully get to the line number.  Everything else is gravy.
							bSuccess |= true;

							// Scroll to Line Number
							TComPtr<EnvDTE::Document> Document;
							TComPtr<IDispatch> SelectionDispatch;
							TComPtr<EnvDTE::TextSelection> Selection;

							int32 RetryCount = 5;
							do
							{
								// Potential exception can be thrown here in the form of 'Call was rejected by callee'
								// See https://msdn.microsoft.com/en-us/library/ms228772.aspx
								if (!SUCCEEDED(DTE->get_ActiveDocument(&Document)) ||
									!SUCCEEDED(Document->get_Selection(&SelectionDispatch)) ||
									!SUCCEEDED(SelectionDispatch->QueryInterface(&Selection)))
								{
									if (RetryCount == 0)
									{
										UE_LOGF(LogVSAccessor, Warning, "Couldn't goto line number '%i' in '%ls'.", Request.LineNumber, *Request.FullPath);
									}
									else
									{
										FPlatformProcess::Sleep(0.1f);
									}
								}

							} while (--RetryCount >= 0);

							if (Selection.IsValid() && !SUCCEEDED(Selection->MoveToLineAndOffset(Request.LineNumber, Request.ColumnNumber, VARIANT_FALSE)))
							{
								if (!SUCCEEDED(Selection->GotoLine(Request.LineNumber, VARIANT_TRUE)))
								{
									UE_LOGF(LogVSAccessor, Warning, "Couldn't goto column number '%i' of line '%i' in '%ls'.", Request.ColumnNumber, Request.LineNumber, *Request.FullPath);
								}
							}
						}
						else
						{
							UE_LOGF(LogVSAccessor, Warning, "Couldn't open file '%ls'.", *Request.FullPath);
						}
					}

					VSLaunchFinished( true );
				}
				else
				{
					UE_LOGF(LogVSAccessor, Log, "Couldn't get item operations. Visual Studio may still be initializing.");
					bDefer = true;
				}
			}
			else
			{
				UE_LOGF(LogVSAccessor, Warning, "Couldn't set focus on Visual Studio.");
			}
		}
		break;

	case EAccessVisualStudioResult::VSInstanceIsNotOpen:
		{
			bDefer = true;

			// We can't process until we're in the main thread, if we aren't initially defer until we are
			if ( IsInGameThread() )
			{
				// If we haven't already attempted to launch VS do so now
				if ( !IsVSLaunchInProgress() )
				{
					// If there's no valid instance of VS running, run one if we have it installed
					if ( !RunVisualStudioAndOpenSolution(SolutionPath) )
					{
						bDefer = false;
					}
					else
					{
						VSLaunchStarted();
					}
				}
			}
		}
		break;

	case EAccessVisualStudioResult::VSInstanceIsBlocked:
		{
			// VS may be open for the solution we want, but we can't query it right now as it's blocked for some reason
			// Defer this operation so we can try it again later should VS become unblocked
			bDefer = true;
		}
		break;

	default:
		// Do nothing if we failed the VS detection, otherwise we could get stuck in a loop of constantly 
		// trying to open a VS instance since we can't detect that one is already running
		bDefer = false;
		break;
	}
	
	if ( !bSuccess )
	{
		// If we have attempted to launch VS, and it's taken too long, timeout so the user can try again
		if ( IsVSLaunchInProgress() && ( FPlatformTime::Seconds() - VSLaunchTime ) > 300 )
		{
			// We need todo this in case the process died or was kill prior to the code gaining focus of it
			bDefer = false;
			VSLaunchFinished(false);

			// We failed to open the solution and file, so lets just use the platforms default opener.
			for ( const FileOpenRequest& Request : Requests )
			{
				FPlatformProcess::LaunchFileInDefaultExternalApplication(*Request.FullPath);
			}
		}

		// Defer the request until VS is available to take hold of
		if ( bDefer )
		{
			FScopeLock Lock(&DeferredRequestsCriticalSection);
			DeferredRequests.Append(Requests);
		}
		else if ( !bSuccess )
		{
			UE_LOGF(LogVSAccessor, Warning, "Couldn't access Visual Studio");
		}
	}

	bWasDeferred = bDefer;
	return bSuccess;
}

bool FVisualStudioSourceCodeAccessor::SaveAllOpenDocuments() const
{
	FCoInitializeScope CoInitialze;
	if (!CoInitialze.IsValid())
	{
		return false;
	}

	bool bSuccess = false;
	
	TComPtr<EnvDTE::_DTE> DTE;
	const FString SolutionPath = GetSolutionPath();
	if (AccessVisualStudioViaDTE(DTE, SolutionPath, GetPrioritizedVisualStudioVersions(SolutionPath)) == EAccessVisualStudioResult::VSInstanceIsOpen)
	{
		// Save all documents
		TComPtr<EnvDTE::Documents> Documents;
		if (SUCCEEDED(DTE->get_Documents(&Documents)) &&
			SUCCEEDED(Documents->SaveAll()))
		{
			bSuccess = true;
		}
		else
		{
			UE_LOGF(LogVSAccessor, Warning, "Couldn't save all documents");
		}
	}
	else
	{
		UE_LOGF(LogVSAccessor, Warning, "Couldn't access Visual Studio");
	}

	return bSuccess;
}

void FVisualStudioSourceCodeAccessor::VSLaunchStarted()
{
	// Broadcast the info and hope that MainFrame is around to receive it
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));
	SourceCodeAccessModule.OnLaunchingCodeAccessor().Broadcast();
	VSLaunchTime = FPlatformTime::Seconds();
}

void FVisualStudioSourceCodeAccessor::VSLaunchFinished( bool bSuccess )
{
	// Finished all requests! Notify the UI.
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));
	SourceCodeAccessModule.OnDoneLaunchingCodeAccessor().Broadcast( bSuccess );
	VSLaunchTime = 0.0;
}

#else

// VS Express-only dummy versions
bool FVisualStudioSourceCodeAccessor::SaveAllOpenDocuments() const { return false; }
void FVisualStudioSourceCodeAccessor::VSLaunchStarted() {}
void FVisualStudioSourceCodeAccessor::VSLaunchFinished( bool bSuccess ) {}

#endif

bool GetProcessCommandLine(const ::DWORD InProcessID, FString& OutCommandLine)
{
	check(InProcessID);

	FCoInitializeScope CoInitialze;
	if (!CoInitialze.IsValid())
	{
		return false;
	}

	bool bSuccess = false;

	::IWbemLocator *pLoc = nullptr;
	if (SUCCEEDED(::CoCreateInstance(CLSID_WbemLocator, 0, CLSCTX_INPROC_SERVER, IID_IWbemLocator, (LPVOID*)&pLoc)))
	{
		FComBSTR ResourceName(TEXT("ROOT\\CIMV2"));

		::IWbemServices *pSvc = nullptr;
		if (SUCCEEDED(pLoc->ConnectServer(ResourceName, nullptr, nullptr, nullptr, 0, 0, 0, &pSvc)))
		{
			// Set the proxy so that impersonation of the client occurs
			if (SUCCEEDED(::CoSetProxyBlanket(pSvc, RPC_C_AUTHN_WINNT, RPC_C_AUTHZ_NONE, nullptr, RPC_C_AUTHN_LEVEL_CALL, RPC_C_IMP_LEVEL_IMPERSONATE, nullptr, EOAC_NONE)))
			{
				::IEnumWbemClassObject* pEnumerator = nullptr;
				const FString WQLQuery = FString::Printf(TEXT("SELECT ProcessId, CommandLine FROM Win32_Process WHERE ProcessId=%lu"), InProcessID);

				FComBSTR WQLBstr(TEXT("WQL"));
				FComBSTR WQLQueryBstr(*WQLQuery);

				if (SUCCEEDED(pSvc->ExecQuery(WQLBstr, WQLQueryBstr, WBEM_FLAG_FORWARD_ONLY | WBEM_FLAG_RETURN_IMMEDIATELY, nullptr, &pEnumerator)))
				{
					while (pEnumerator && !bSuccess)
					{
						::IWbemClassObject *pclsObj = nullptr;
						::ULONG uReturn = 0;
						if (SUCCEEDED(pEnumerator->Next(WBEM_INFINITE, 1, &pclsObj, &uReturn)))
						{
							if (uReturn == 1)
							{
								::VARIANT vtProp;

								::DWORD CurProcessID = 0;
								if (SUCCEEDED(pclsObj->Get(TEXT("ProcessId"), 0, &vtProp, 0, 0)))
								{
									CurProcessID = vtProp.ulVal;
									::VariantClear(&vtProp);
								}

								check(CurProcessID == InProcessID);
								if (SUCCEEDED(pclsObj->Get(TEXT("CommandLine"), 0, &vtProp, 0, 0)))
								{
									OutCommandLine = vtProp.bstrVal;
									::VariantClear(&vtProp);

									bSuccess = true;
								}

								pclsObj->Release();
							}
						}
					}

					pEnumerator->Release();
				}
			}

			pSvc->Release();
		}

		pLoc->Release();
	}

	return bSuccess;
}

::HWND GetTopWindowForProcess(const ::DWORD InProcessID)
{
	check(InProcessID);

	struct EnumWindowsData
	{
		::DWORD InProcessID;
		::HWND OutHwnd;

		static ::BOOL CALLBACK EnumWindowsProc(::HWND Hwnd, ::LPARAM lParam)
		{
			EnumWindowsData *const Data = (EnumWindowsData*)lParam;

			::DWORD HwndProcessId = 0;
			::GetWindowThreadProcessId(Hwnd, &HwndProcessId);

			if (HwndProcessId == Data->InProcessID)
			{
				Data->OutHwnd = Hwnd;
				return 0; // stop enumeration
			}
			return 1; // continue enumeration
		}
	};

	EnumWindowsData Data;
	Data.InProcessID = InProcessID;
	Data.OutHwnd = nullptr;
	::EnumWindows(&EnumWindowsData::EnumWindowsProc, (LPARAM)&Data);

	return Data.OutHwnd;
}

/** Query all opened instances of Visual Studio to retrieve their respective opened solutions. */
TArray<FString> RetrieveOpenedVisualStudioSolutionsViaProcess(const TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation>& InLocations)
{
	TArray<FString> OpenedSolutions;

	::HANDLE hProcessSnap = ::CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);
	if (hProcessSnap != INVALID_HANDLE_VALUE)
	{
		MODULEENTRY32 ModuleEntry;
		ModuleEntry.dwSize = sizeof(MODULEENTRY32);

		PROCESSENTRY32 ProcEntry;
		ProcEntry.dwSize = sizeof(PROCESSENTRY32);

		// We enumerate the locations as the outer loop to ensure we find our preferred process type first
		// If we did this as the inner loop, then we'd get the first process that matched any location, even if it wasn't our preference
		for (const auto& Location : InLocations)
		{
			for (::BOOL bHasProcess = ::Process32First(hProcessSnap, &ProcEntry); bHasProcess; bHasProcess = ::Process32Next(hProcessSnap, &ProcEntry))
			{
				::HANDLE hModuleSnap = ::CreateToolhelp32Snapshot(TH32CS_SNAPMODULE, ProcEntry.th32ProcessID);
				if (hModuleSnap != INVALID_HANDLE_VALUE)
				{
					if (::Module32First(hModuleSnap, &ModuleEntry))
					{
						FString ProcPath = ModuleEntry.szExePath;

						if (ProcPath == Location.ExecutablePath)
						{
							// Without DTE we can't accurately verify that the Visual Studio instance has the correct solution open,
							// however, if we've opened it (or it's opened the solution directly), then the solution path will
							// exist somewhere in the command line for the process
							FString CommandLine;
							if (GetProcessCommandLine(ProcEntry.th32ProcessID, CommandLine))
							{
								TArray<FString> Tokens;
								CommandLine.ParseIntoArray(Tokens, TEXT(" "), /*InCullEmpty = */true);
								FString SolutionName;
								TArray<FString> SolutionNames;
								Algo::TransformIf(Tokens, SolutionNames, 
									[](const FString& InToken) { return InToken.EndsWith(TEXT(".sln"), ESearchCase::IgnoreCase) || InToken.EndsWith(TEXT(".slnf"), ESearchCase::IgnoreCase); },
									[](const FString& InToken) 
									{ 
										FString Filename = InToken.TrimStartAndEnd();
										Filename.RemoveFromStart(TEXT("\""));
										Filename.RemoveFromEnd(TEXT("\""));
										FPaths::NormalizeFilename(Filename);
										return Filename;
									});
								
								if (!SolutionNames.IsEmpty())
								{
									OpenedSolutions.AddUnique(SolutionNames[0]);
								}
							}
							else
							{
								UE_LOGF(LogVSAccessor, Warning, "Couldn't access module information");
							}
						}
					}
					else
					{
						UE_LOGF(LogVSAccessor, Warning, "Couldn't access module table");
					}

					::CloseHandle(hModuleSnap);
				}
				else
				{
					UE_LOGF(LogVSAccessor, Warning, "Couldn't access module table");
				}
			}
		}

		::CloseHandle(hProcessSnap);
	}
	else
	{
		UE_LOGF(LogVSAccessor, Warning, "Couldn't access process table");
	}

	return OpenedSolutions;
}

EAccessVisualStudioResult AccessVisualStudioViaProcess(::DWORD& OutProcessID, FString& OutExecutablePath, const FString& InSolutionPath, const TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation>& InLocations)
{
	OutProcessID = 0;

	EAccessVisualStudioResult AccessResult = EAccessVisualStudioResult::VSInstanceIsNotOpen;

	::HANDLE hProcessSnap = ::CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);
	if (hProcessSnap != INVALID_HANDLE_VALUE)
	{
		MODULEENTRY32 ModuleEntry;
		ModuleEntry.dwSize = sizeof(MODULEENTRY32);

		PROCESSENTRY32 ProcEntry;
		ProcEntry.dwSize = sizeof(PROCESSENTRY32);

		// We enumerate the locations as the outer loop to ensure we find our preferred process type first
		// If we did this as the inner loop, then we'd get the first process that matched any location, even if it wasn't our preference
		for (const auto& Location : InLocations)
		{
			for (::BOOL bHasProcess = ::Process32First(hProcessSnap, &ProcEntry); bHasProcess && !OutProcessID; bHasProcess = ::Process32Next(hProcessSnap, &ProcEntry))
			{
				::HANDLE hModuleSnap = ::CreateToolhelp32Snapshot(TH32CS_SNAPMODULE, ProcEntry.th32ProcessID);
				if (hModuleSnap != INVALID_HANDLE_VALUE)
				{
					if (::Module32First(hModuleSnap, &ModuleEntry))
					{
						FString ProcPath = ModuleEntry.szExePath;

						if (ProcPath == Location.ExecutablePath)
						{
							// Without DTE we can't accurately verify that the Visual Studio instance has the correct solution open,
							// however, if we've opened it (or it's opened the solution directly), then the solution path will
							// exist somewhere in the command line for the process
							FString CommandLine;
							if (GetProcessCommandLine(ProcEntry.th32ProcessID, CommandLine))
							{
								FPaths::NormalizeFilename(CommandLine);
								if (CommandLine.Contains(InSolutionPath))
								{
									OutProcessID = ProcEntry.th32ProcessID;
									OutExecutablePath = Location.ExecutablePath;
									AccessResult = EAccessVisualStudioResult::VSInstanceIsOpen;
									break;
								}
							}
							else
							{
								UE_LOGF(LogVSAccessor, Warning, "Couldn't access module information");
								AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
							}
						}
					}
					else
					{
						UE_LOGF(LogVSAccessor, Warning, "Couldn't access module table");
						AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
					}

					::CloseHandle(hModuleSnap);
				}
				else
				{
					UE_LOGF(LogVSAccessor, Warning, "Couldn't access module table");
					AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
				}
			}
		}

		::CloseHandle(hProcessSnap);
	}
	else
	{
		UE_LOGF(LogVSAccessor, Warning, "Couldn't access process table");
		AccessResult = EAccessVisualStudioResult::VSInstanceUnknown;
	}

	return AccessResult;
}

bool FVisualStudioSourceCodeAccessor::OpenSolution()
{
#if WITH_VISUALSTUDIO_DTE
	if(OpenVisualStudioSolutionViaDTE())
	{
		return true;
	}
#endif
	return OpenVisualStudioSolutionViaProcess();
}

bool FVisualStudioSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	bool bSuccess = false;

	{
		FScopeLock Lock(&CachedSolutionPathCriticalSection);
		CachedSolutionPathOverride = InSolutionPath;
	}
#if WITH_VISUALSTUDIO_DTE
	if (OpenVisualStudioSolutionViaDTE())
	{
		bSuccess = true;
	}
	else
#endif
	{
		bSuccess = OpenVisualStudioSolutionViaProcess();
	}

	{
		FScopeLock Lock(&CachedSolutionPathCriticalSection);
		CachedSolutionPathOverride = TEXT("");
	}
	return bSuccess;
}

bool FVisualStudioSourceCodeAccessor::DoesSolutionExist() const
{
	const FString SolutionPath = GetSolutionPath();
	return FPaths::FileExists(SolutionPath) || FPaths::DirectoryExists(SolutionPath);
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioFilesInternal(const TArray<FileOpenRequest>& Requests)
{
#if WITH_VISUALSTUDIO_DTE
	{
		bool bWasDeferred = false;
		if(OpenVisualStudioFilesInternalViaDTE(Requests, bWasDeferred) || bWasDeferred)
		{
			return true;
		}
	}
#endif
	return OpenVisualStudioFilesInternalViaProcess(Requests);
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioSolutionViaProcess()
{
	::DWORD ProcessID = 0;
	FString Path;
	const FString SolutionPath = GetSolutionPath();
	switch (AccessVisualStudioViaProcess(ProcessID, Path, SolutionPath, GetPrioritizedVisualStudioVersions(SolutionPath)))
	{
		case EAccessVisualStudioResult::VSInstanceIsOpen:
		{
			// Try to bring Visual Studio to the foreground
			::HWND VisualStudioHwnd = GetTopWindowForProcess(ProcessID);
			if (VisualStudioHwnd)
			{
				// SwitchToThisWindow isn't really intended for general use, however it can switch to 
				// the VS window, where SetForegroundWindow will fail due to process permissions
				::SwitchToThisWindow(VisualStudioHwnd, 0);
			}
			return true;
		}

	case EAccessVisualStudioResult::VSInstanceIsNotOpen:
		return RunVisualStudioAndOpenSolution(SolutionPath);

	default:
		// Do nothing if we failed the VS detection, otherwise we could get stuck in a loop of constantly 
		// trying to open a VS instance since we can't detect that one is already running
		break;
	}

		return false;
}

FString FVisualStudioSourceCodeAccessor::RetrieveSolutionForFileOpenRequests(const TArray<FileOpenRequest>& Requests, const TArray<FString>& CurrentlyOpenedSolutions) const
{
	// Based on the files being requested, make an educated guess as to which is the most appropriate solution to open them all by finding the corresponding .sln/.slnf files in the folder hierarchy: 
	struct FSolutionInfo
	{
		FSolutionInfo(const FString& InSolutionFile)
			: SolutionFile(InSolutionFile)
		{}

		// Describes the state of a solution file wrt the currently opened solutions (ordered by priority)
		enum class EOpenedSolutionState : uint8
		{
			CurrentlyOpenedExactMatch = 0, // The solution is currently opened in visual studio
			CurrentlyOpened, // A solution with same file name but not the same absolute path is currently opened in visual studio
			NotOpened, // The solution is not currently opened in Visual Studio
		};
		EOpenedSolutionState OpenedSolutionState = FSolutionInfo::EOpenedSolutionState::NotOpened;
		int32 RefCount = 0;
		FString SolutionFile;
	};

	const bool bUseUProject = GetDefault<UVisualStudioSourceCodeAccessSettings>()->bUproject;

	TArray<FSolutionInfo> SolutionFileInfos;
	for (const FileOpenRequest& Request : Requests)
	{
		FString CurrentPath = FPaths::GetPath(Request.FullPath);
		while (!CurrentPath.IsEmpty())
		{
			if (bUseUProject)
			{
				TArray<FString> FilesInDirectory;
				IFileManager::Get().FindFiles(FilesInDirectory, *CurrentPath, TEXT(".uproject"));
				if (FilesInDirectory.Num() > 0)
				{
					FSolutionInfo* SolutionInfo = SolutionFileInfos.FindByPredicate([&CurrentPath](const FSolutionInfo& InSolutionInfo) { return InSolutionInfo.SolutionFile == CurrentPath; });
					if (SolutionInfo == nullptr)
					{
						SolutionInfo = &SolutionFileInfos.Add_GetRef(FSolutionInfo(CurrentPath));

						for (const FString& OpenedSolution : CurrentlyOpenedSolutions)
						{
							if (OpenedSolution.Equals(CurrentPath, ESearchCase::IgnoreCase))
							{
								SolutionInfo->OpenedSolutionState = FSolutionInfo::EOpenedSolutionState::CurrentlyOpenedExactMatch;
								break;
							}
							if (FPaths::GetCleanFilename(OpenedSolution).Equals(CurrentPath, ESearchCase::IgnoreCase))
							{
								SolutionInfo->OpenedSolutionState = FSolutionInfo::EOpenedSolutionState::CurrentlyOpened;
							}
						}
					}
					++SolutionInfo->RefCount;
				}
			}
			else
			{
				TArray<FString> FilesInDirectory;
				IFileManager::Get().FindFiles(FilesInDirectory, *CurrentPath, TEXT(".sln"));
				IFileManager::Get().FindFiles(FilesInDirectory, *CurrentPath, TEXT(".slnf"));
				for (const FString& FileInDirectory : FilesInDirectory)
				{
					FString AbsoluteFileName = CurrentPath / FileInDirectory;
					FPaths::NormalizeFilename(AbsoluteFileName);

					FSolutionInfo* SolutionInfo = SolutionFileInfos.FindByPredicate([&AbsoluteFileName](const FSolutionInfo& InSolutionInfo) { return InSolutionInfo.SolutionFile == AbsoluteFileName; });
					if (SolutionInfo == nullptr)
					{
						SolutionInfo = &SolutionFileInfos.Add_GetRef(FSolutionInfo(AbsoluteFileName));

						for (const FString& OpenedSolution : CurrentlyOpenedSolutions)
						{
							if (OpenedSolution.Equals(AbsoluteFileName, ESearchCase::IgnoreCase))
							{
								SolutionInfo->OpenedSolutionState = FSolutionInfo::EOpenedSolutionState::CurrentlyOpenedExactMatch;
								break;
							}
							if (FPaths::GetCleanFilename(OpenedSolution).Equals(FileInDirectory, ESearchCase::IgnoreCase))
							{
								SolutionInfo->OpenedSolutionState = FSolutionInfo::EOpenedSolutionState::CurrentlyOpened;
							}
						}
					}
					++SolutionInfo->RefCount;
				}
			}
			CurrentPath = FPaths::GetPath(CurrentPath);
		}
	}

	// Now that we have a list of all solutions that could be used to open all these files, pick the best one : 
	SolutionFileInfos.Sort([](const FSolutionInfo& InLHS, const FSolutionInfo& InRHS)
	{
		// Sort by ref count first, so that the most requested solution comes out on top : 
		if (InLHS.RefCount < InRHS.RefCount)
		{
			return false;
		}
		else if (InLHS.RefCount == InRHS.RefCount)
		{
			return (static_cast<uint8>(InLHS.OpenedSolutionState) < static_cast<uint8>(InRHS.OpenedSolutionState));
		}

		return true;
	});

	return SolutionFileInfos.IsEmpty() ? FString() : SolutionFileInfos[0].SolutionFile;
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioFilesInternalViaProcess(const TArray<FileOpenRequest>& Requests)
{
	::DWORD ProcessID = 0;
	FString Path;
	FString SolutionPath = GetSolutionPath();

	TArray<VisualStudioLocation> InstalledLocations = GetPrioritizedVisualStudioVersions(SolutionPath);

	// Even if a solution is specified, also check open solutions and make an educated guess about the best path based on the files being requested by finding the corresponding .sln files: 
	TArray<FString> CurrentlyOpenedSolutions = RetrieveOpenedVisualStudioSolutionsViaProcess(InstalledLocations);
	if (!SolutionPath.IsEmpty())
	{
		CurrentlyOpenedSolutions.Add(SolutionPath);
	}
	SolutionPath = RetrieveSolutionForFileOpenRequests(Requests, CurrentlyOpenedSolutions);

	switch (AccessVisualStudioViaProcess(ProcessID, Path, SolutionPath, InstalledLocations))
	{
	case EAccessVisualStudioResult::VSInstanceIsOpen:
		return RunVisualStudioAndOpenSolutionAndFiles(Path, "", &Requests);
	
	case EAccessVisualStudioResult::VSInstanceIsNotOpen:
		if (CanRunVisualStudio(Path, SolutionPath))
		{
			return RunVisualStudioAndOpenSolutionAndFiles(Path, SolutionPath, &Requests);
		}
		break;

	default:
		// Do nothing if we failed the VS detection, otherwise we could get stuck in a loop of constantly 
		// trying to open a VS instance since we can't detect that one is already running
		break;
	}

	return false;
}

bool FVisualStudioSourceCodeAccessor::CanRunVisualStudio(FString& OutPath, const FString& InSolution) const
{
	TArray<VisualStudioLocation> PrioritizedLocations = GetPrioritizedVisualStudioVersions(InSolution);
	if (PrioritizedLocations.Num() > 0)
	{
		OutPath = PrioritizedLocations[0].ExecutablePath;
		return true;
	}

	return false;
}

bool FVisualStudioSourceCodeAccessor::RunVisualStudioAndOpenSolution(const FString& InSolution) const
{
	FString Path;
	if (CanRunVisualStudio(Path, InSolution))
	{
		return RunVisualStudioAndOpenSolutionAndFiles(Path, InSolution, nullptr);
	}

	return false;
}

bool FVisualStudioSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	// Automatically fail if there's already an attempt in progress
	if ( !IsVSLaunchInProgress() )
	{
		TArray<FileOpenRequest> Requests;
		for ( const FString& FullPath : AbsoluteSourcePaths )
		{
			Requests.Add(FileOpenRequest(FullPath, 1, 1));
		}

		return OpenVisualStudioFilesInternal(Requests);
	}

	return false;
}

bool FVisualStudioSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	bool bSuccess = false;

	// This code is temporarily disabled because it doesn't account for UBT setting per-file properties for C++ source files,
	// adding include paths, force-included headers, and so on. Intellisense does not work correctly without these properties being set.
#if 0
	// This requires DTE - there is no fallback for this operation when DTE is not available
#if WITH_VISUALSTUDIO_DTE
	bSuccess = true;
	struct FModuleNameAndPath
	{
		FString ModuleBuildFilePath;
		FString ModulePath;
		FName ModuleName;
	};

	TArray<FModuleNameAndPath> ModuleNamesAndPaths;
	ModuleNamesAndPaths.Reserve(AvailableModules.Num());
	for (const FString& AvailableModule : AvailableModules)
	{
		static const int32 BuildFileExtensionLen = FString(TEXT(".Build.cs")).Len();

		// AvailableModule is the relative path to the .Build.cs file
		FModuleNameAndPath ModuleNameAndPath;
		ModuleNameAndPath.ModuleBuildFilePath = FPaths::ConvertRelativePathToFull(AvailableModule);
		ModuleNameAndPath.ModulePath = FPaths::GetPath(ModuleNameAndPath.ModuleBuildFilePath);
		ModuleNameAndPath.ModuleName = *FPaths::GetCleanFilename(ModuleNameAndPath.ModuleBuildFilePath).LeftChop(BuildFileExtensionLen);
		ModuleNamesAndPaths.Add(ModuleNameAndPath);
	}

	struct FModuleNewSourceFiles
	{
		FModuleNameAndPath ModuleNameAndPath;
		TArray<FString> NewSourceFiles;
	};

	// Work out which module each source file will be in
	TMap<FName, FModuleNewSourceFiles> ModuleToNewSourceFiles;
	{
		const FModuleNameAndPath* LastSourceFilesModule = nullptr;
		for (const FString& SourceFile : AbsoluteSourcePaths)
		{
			// First check to see if this source file is in the same module as the last source file - this is usually the case, and saves us a lot of string compares
			if (LastSourceFilesModule && SourceFile.StartsWith(LastSourceFilesModule->ModulePath + "/"))
			{
				FModuleNewSourceFiles& ModuleNewSourceFiles = ModuleToNewSourceFiles.FindChecked(LastSourceFilesModule->ModuleName);
				ModuleNewSourceFiles.NewSourceFiles.Add(SourceFile);
				continue;
			}

			// Look for the module which will contain this file
			LastSourceFilesModule = nullptr;
			for (const FModuleNameAndPath& ModuleNameAndPath : ModuleNamesAndPaths)
			{
				if (SourceFile.StartsWith(ModuleNameAndPath.ModulePath + "/"))
				{
					LastSourceFilesModule = &ModuleNameAndPath;

					FModuleNewSourceFiles& ModuleNewSourceFiles = ModuleToNewSourceFiles.FindOrAdd(ModuleNameAndPath.ModuleName);
					ModuleNewSourceFiles.ModuleNameAndPath = ModuleNameAndPath;
					ModuleNewSourceFiles.NewSourceFiles.Add(SourceFile);
					break;
				}
			}

			// Failed to find the module for this source file?
			if (!LastSourceFilesModule)
			{
				UE_LOGF(LogVSAccessor, Warning, "Cannot add source file '%ls' as it doesn't belong to a known module", *SourceFile);
				bSuccess = false;
			}
		}
	}

	TComPtr<EnvDTE::_DTE> DTE;
	const FString SolutionPath = GetSolutionPath();
	if (AccessVisualStudioViaDTE(DTE, SolutionPath, GetPrioritizedVisualStudioVersions(SolutionPath)) == EAccessVisualStudioResult::VSInstanceIsOpen)
	{
		TComPtr<EnvDTE::_Solution> Solution;
		if (SUCCEEDED(DTE->get_Solution(&Solution)) && Solution)
		{
			// Process each module
			for (const auto& ModuleNewSourceFilesKeyValue : ModuleToNewSourceFiles)
			{
				const FModuleNewSourceFiles& ModuleNewSourceFiles = ModuleNewSourceFilesKeyValue.Value;

				const FString& ModuleBuildFilePath = ModuleNewSourceFiles.ModuleNameAndPath.ModuleBuildFilePath;
				auto ANSIModuleBuildFilePath = StringCast<ANSICHAR>(*ModuleBuildFilePath);
				FComBSTR COMStrModuleBuildFilePath(ANSIModuleBuildFilePath.Get());

				TComPtr<EnvDTE::ProjectItem> BuildFileProjectItem;
				if (SUCCEEDED(Solution->FindProjectItem(COMStrModuleBuildFilePath, &BuildFileProjectItem)) && BuildFileProjectItem)
				{
					// We found the .Build.cs file in the existing solution - now we need its parent ProjectItems as that's what we'll be adding new content to
					TComPtr<EnvDTE::ProjectItems> ModuleProjectFolder;
					if (SUCCEEDED(BuildFileProjectItem->get_Collection(&ModuleProjectFolder)) && ModuleProjectFolder)
					{
						for (const FString& SourceFile : AbsoluteSourcePaths)
						{
							const FString ProjectRelativeSourceFilePath = SourceFile.Mid(ModuleNewSourceFiles.ModuleNameAndPath.ModulePath.Len());
							TArray<FString> SourceFileParts;
							ProjectRelativeSourceFilePath.ParseIntoArray(SourceFileParts, TEXT("/"), true);
					
							if (SourceFileParts.Num() == 0)
							{
								// This should never happen as it means we somehow have no filename within the project directory
								bSuccess = false;
								continue;
							}

							TComPtr<EnvDTE::ProjectItems> CurProjectItems = ModuleProjectFolder;

							// Firstly we need to make sure that all the folders we need exist - this also walks us down to the correct place to add the file
							for (int32 FilePartIndex = 0; FilePartIndex < SourceFileParts.Num() - 1 && CurProjectItems; ++FilePartIndex)
							{
								const FString& SourceFilePart = SourceFileParts[FilePartIndex];

								auto ANSIPart = StringCast<ANSICHAR>(*SourceFilePart);
								FComBSTR COMStrFilePart(ANSIPart.Get());

								::VARIANT vProjectItemName;
								vProjectItemName.vt = VT_BSTR;
								vProjectItemName.bstrVal = COMStrFilePart;

								TComPtr<EnvDTE::ProjectItem> ProjectItem;
								if (SUCCEEDED(CurProjectItems->Item(vProjectItemName, &ProjectItem)) && !ProjectItem)
								{
									// Add this part
									CurProjectItems->AddFolder(COMStrFilePart, nullptr, &ProjectItem);
								}

								if (ProjectItem)
								{
									ProjectItem->get_ProjectItems(&CurProjectItems);
								}
								else
								{
									CurProjectItems = nullptr;
								}
							}

							if (!CurProjectItems)
							{
								// Failed to find or add all the path parts
								bSuccess = false;
								continue;
							}

							// Now we add the file to the project under the last folder we found along its path
							auto ANSIPath = StringCast<ANSICHAR>(*SourceFile);
							FComBSTR COMStrFileName(ANSIPath.Get());
							TComPtr<EnvDTE::ProjectItem> FileProjectItem;
							if (SUCCEEDED(CurProjectItems->AddFromFile(COMStrFileName, &FileProjectItem)))
							{
								bSuccess &= true;
							}
						}

						// Save the updated project to avoid a message when closing VS
						TComPtr<EnvDTE::Project> Project;
						if (SUCCEEDED(ModuleProjectFolder->get_ContainingProject(&Project)) && Project)
						{
							Project->Save(nullptr);
						}
					}
					else
					{
						UE_LOGF(LogVSAccessor, Warning, "Cannot add source files as we failed to get the parent items container for the '%ls' item", *ModuleBuildFilePath);
						bSuccess = false;
					}
				}
				else
				{
					UE_LOGF(LogVSAccessor, Warning, "Cannot add source files as we failed to find '%ls' in the solution", *ModuleBuildFilePath);
					bSuccess = false;
				}
			}
		}
		else
		{
			UE_LOGF(LogVSAccessor, Warning, "Cannot add source files as Visual Studio failed to return a solution when queried");
			bSuccess = false;
		}
	}
	else
	{
		UE_LOGF(LogVSAccessor, Verbose, "Cannot add source files as Visual Studio is either not open or not responding");
		bSuccess = false;
	}
#endif
#else
	// if we add new source files but do not add them directly and rely on project generation instead, if we have an opened instance, request for the files to be opened now
	// this is because project generation will trigger a modal on our opened instance and prevent file open request to be handled.
	OpenSourceFiles(AbsoluteSourcePaths);
#endif

	return bSuccess;
}

bool FVisualStudioSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	// column & line numbers are 1-based, so dont allow zero
	if(LineNumber == 0)
	{
		LineNumber++;
	}
	if(ColumnNumber == 0)
{
		ColumnNumber++;
	}

	// Automatically fail if there's already an attempt in progress
	if (!IsVSLaunchInProgress())
	{
		return OpenVisualStudioFileAtLineInternal(FullPath, LineNumber, ColumnNumber);
	}

	return false;
}

bool FVisualStudioSourceCodeAccessor::OpenVisualStudioFileAtLineInternal(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	TArray<FileOpenRequest> Requests;
	Requests.Add(FileOpenRequest(FullPath, LineNumber, ColumnNumber));

	return OpenVisualStudioFilesInternal(Requests);
}

void FVisualStudioSourceCodeAccessor::AddVisualStudioVersion(const int MajorVersion, const bool bAllowExpress)
{
	FString CommonToolsPath;
	if (!FPlatformMisc::GetVSComnTools(MajorVersion, CommonToolsPath))
	{
		return;
	}

	FString BaseExecutablePath = FPaths::Combine(*CommonToolsPath, TEXT(".."), TEXT("IDE"));
	FPaths::NormalizeDirectoryName(BaseExecutablePath);
	FPaths::CollapseRelativeDirectories(BaseExecutablePath);

	VisualStudioLocation NewLocation;
	NewLocation.VersionNumber = MajorVersion;
	NewLocation.bPreviewRelease = false;
	NewLocation.ExecutablePath = BaseExecutablePath / TEXT("devenv.exe");
#if WITH_VISUALSTUDIO_DTE
	NewLocation.ROTMoniker = FString::Printf(TEXT("!VisualStudio.DTE.%d.0"), MajorVersion);
#endif

	// Only add this version of Visual Studio if the devenv executable actually exists
	const bool bDevEnvExists = FPaths::FileExists(NewLocation.ExecutablePath);
	if (bDevEnvExists)
	{
		Locations.Add(NewLocation);
	}

	if (bAllowExpress)
	{
		NewLocation.ExecutablePath = BaseExecutablePath / TEXT("WDExpress.exe");
#if WITH_VISUALSTUDIO_DTE
		NewLocation.ROTMoniker = FString::Printf(TEXT("!WDExpress.DTE.%d.0"), MajorVersion);
#endif

		// Only add this version of Visual Studio if the WDExpress executable actually exists
		const bool bWDExpressExists = FPaths::FileExists(NewLocation.ExecutablePath);
		if (bWDExpressExists)
		{
			Locations.Add(NewLocation);
		}
	}
}

void FVisualStudioSourceCodeAccessor::AddVisualStudioVersionUsingVisualStudioSetupAPI(int MinimumVersionNumber)
{
	FCoInitializeScope CoInitialze;
	if (!CoInitialze.IsValid())
	{
		return;
	}

	// Try to create the CoCreate the class; if that fails, likely no instances are registered.
	TComPtr<ISetupConfiguration2> Query;
	HRESULT Result = CoCreateInstance(__uuidof(SetupConfiguration), nullptr, CLSCTX_ALL, __uuidof(ISetupConfiguration2), (LPVOID*)&Query);
	if (FAILED(Result))
	{
		UE_LOGF(LogVSAccessor, Display, "Unable to create Visual Studio setup instance: %08x", Result);
		return;
	}

	// Get the enumerator
	TComPtr<IEnumSetupInstances> EnumSetupInstances;
	Result = Query->EnumAllInstances(&EnumSetupInstances);
	if (FAILED(Result))
	{
		UE_LOGF(LogVSAccessor, Warning, "Unable to query Visual Studio setup instances: %08x", Result);
		return;
	}

	// Check the state and version of the enumerated instances
	TComPtr<ISetupInstance> Instance;
	for (;;)
	{
		unsigned long NumFetched = 0;
		Result = EnumSetupInstances->Next(1, &Instance, &NumFetched);
		if (FAILED(Result) || NumFetched == 0)
		{
			break;
		}

		TComPtr<ISetupInstance2> Instance2;
		Result = Instance->QueryInterface(__uuidof(ISetupInstance2), (LPVOID*)&Instance2);
		if (SUCCEEDED(Result))
		{
			InstanceState State;
			Result = Instance2->GetState(&State);
			if (SUCCEEDED(Result) && (State & eLocal) != 0)
			{
				BSTR InstallationVersion;
				Result = Instance2->GetInstallationVersion(&InstallationVersion);
				if (SUCCEEDED(Result))
				{
					int Major, Minor, Build, Patch;
					int ScanResult = swscanf_s(InstallationVersion, TEXT("%d.%d.%d.%d"), &Major, &Minor, &Build, &Patch);
					if (ScanResult == 4 && Major >= MinimumVersionNumber)
					{
						BSTR InstallationPath;
						Result = Instance2->GetInstallationPath(&InstallationPath);
						if (SUCCEEDED(Result))
						{
							BSTR ProductPath;
							Result = Instance2->GetProductPath(&ProductPath);
							if (SUCCEEDED(Result))
							{
								VisualStudioLocation NewLocation;
								NewLocation.VersionNumber = Major;
								NewLocation.bPreviewRelease = false;
								NewLocation.ExecutablePath = FString::Printf(TEXT("%s\\%s"), InstallationPath, ProductPath);
#if WITH_VISUALSTUDIO_DTE
								NewLocation.ROTMoniker = FString::Printf(TEXT("!VisualStudio.DTE.%d.0"), Major);
#endif

								TComPtr<ISetupInstanceCatalog> Catalog;
								Result = Instance2->QueryInterface(__uuidof(ISetupInstanceCatalog), (LPVOID*)&Catalog);
								if (SUCCEEDED(Result) && Catalog != nullptr)
								{
									VARIANT_BOOL PrereleaseFlag;
									Result = Catalog->IsPrerelease(&PrereleaseFlag);
									if (SUCCEEDED(Result) && PrereleaseFlag == VARIANT_TRUE)
									{
										NewLocation.bPreviewRelease = true;
									}
								}

								Locations.Add(NewLocation);

								SysFreeString(ProductPath);
							}
							SysFreeString(InstallationPath);
						}
					}
					SysFreeString(InstallationVersion);
				}
			}
		}
	}
}

int32 FVisualStudioSourceCodeAccessor::VisualStudioVersionSortWeight(const VisualStudioLocation& InLocation, const int32 ExactVersion, const bool bPreferPreview) const
{
	return
		// First sort by VersionNumber. If the version matches ExactVersion it should be prioritized
		(InLocation.VersionNumber == ExactVersion ? InLocation.VersionNumber * 100 : InLocation.VersionNumber * 10)
		// Then by if a preview release is preferred or not
		+ (InLocation.bPreviewRelease == bPreferPreview ? 1 : 0);
}

TArray<FVisualStudioSourceCodeAccessor::VisualStudioLocation> FVisualStudioSourceCodeAccessor::GetPrioritizedVisualStudioVersions(const FString& InSolution) const
{
	TArray<VisualStudioLocation> PrioritizedLocations = Locations;

	int32 SolutionVersion = GetVisualStudioVersionForSolution(InSolution);
	if (SolutionVersion == 0)
	{
		SolutionVersion = GetVisualStudioVersionForCompiler();
	}

	const bool bPreferPreview = GetDefault<UVisualStudioSourceCodeAccessSettings>()->bPreview;

	PrioritizedLocations.StableSort([&](const VisualStudioLocation& InFirst, const VisualStudioLocation& InSecond) -> bool
	{
		return VisualStudioVersionSortWeight(InFirst, SolutionVersion, bPreferPreview) >= VisualStudioVersionSortWeight(InSecond, SolutionVersion, bPreferPreview);
	});
	
	return PrioritizedLocations;
}

bool FVisualStudioSourceCodeAccessor::RunVisualStudioAndOpenSolutionAndFiles(const FString& ExecutablePath, const FString& SolutionPath, const TArray<FileOpenRequest>* const Requests) const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));

	FString Params;

	// Only open the solution if it exists
	if (!SolutionPath.IsEmpty())
	{
		if (FPaths::FileExists(SolutionPath) || FPaths::DirectoryExists(SolutionPath))
		{
			Params += TEXT("\"");
			Params += SolutionPath;
			Params += TEXT("\"");
		}
		else
		{
			SourceCodeAccessModule.OnOpenFileFailed().Broadcast(SolutionPath);
			return false;
		}
	}

	if (Requests)
	{
		int32 GoToLine = 0;
		for (const FileOpenRequest& Request : *Requests)
		{
			// Only open the file if it exists
			if (FPaths::FileExists(Request.FullPath))
			{
				Params += TEXT(" \"");
				FString PlatformFilename = Request.FullPath;
				FPaths::MakePlatformFilename(PlatformFilename);
				Params += PlatformFilename;
				Params += TEXT("\"");

				GoToLine = Request.LineNumber;
			}
			else
			{
				SourceCodeAccessModule.OnOpenFileFailed().Broadcast(Request.FullPath);
				return false;
			}
		}

		if (GoToLine > 0)
		{
			Params += FString::Printf(TEXT(" /command \"edit.goto %d\""), GoToLine);
		}
	}

	FProcHandle WorkerHandle = FPlatformProcess::CreateProc(*ExecutablePath, *Params, true, false, false, nullptr, 0, nullptr, nullptr);
	bool bSuccess = WorkerHandle.IsValid();
	FPlatformProcess::CloseProc(WorkerHandle);
	return bSuccess;
}

bool FVisualStudioSourceCodeAccessor::CanAccessSourceCode() const
{
	// True if we have any versions of VS installed
	return Locations.Num() > 0;
}

FName FVisualStudioSourceCodeAccessor::GetFName() const
{
	return FName("VisualStudioSourceCodeAccessor");
}

FText FVisualStudioSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("VisualStudioDisplayName", "Visual Studio");
}

FText FVisualStudioSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("VisualStudioDisplayDesc", "Open source code files in Visual Studio");
}

FString FVisualStudioSourceCodeAccessor::GetSolutionPath() const
{
	FScopeLock Lock(&CachedSolutionPathCriticalSection);

	if(IsInGameThread())
	{
		const bool bUseUProject = GetDefault<UVisualStudioSourceCodeAccessSettings>()->bUproject;
		if (CachedSolutionPathOverride.Len() > 0)
		{
			if (bUseUProject)
			{
				CachedSolutionPath = FPaths::GetPath(CachedSolutionPathOverride);
				FPaths::NormalizeDirectoryName(CachedSolutionPath);
			}
			else
			{
				CachedSolutionPath = CachedSolutionPathOverride + TEXT(".sln");
				FPaths::NormalizeFilename(CachedSolutionPath);
			}
		}
		else
		{
			if (bUseUProject)
			{
				const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
				if (CurrentProject != nullptr)
				{
					// VS support is implemented to open the directory that contains the .uproject
					CachedSolutionPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
					FPaths::NormalizeDirectoryName(CachedSolutionPath);
					return CachedSolutionPath;
				}
			}
			
			FString PrimaryProjectPath;
			if (!FFileHelper::LoadFileToString(PrimaryProjectPath, *(FPaths::EngineIntermediateDir() / TEXT("ProjectFiles") / TEXT("PrimaryProjectPath.txt"))))
			{
				const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
				const FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

				if ((CurrentProject == nullptr || CurrentProject->Modules.Num() == 0) || !FUProjectDictionary::GetDefault().IsForeignProject(ProjectDir))
				{
					PrimaryProjectPath = FPaths::RootDir() / TEXT("UE5");
				}
				else
				{
					const FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(ProjectDir);
					PrimaryProjectPath = ProjectDir / BaseName;
				}
			}
			CachedSolutionPath = PrimaryProjectPath + TEXT(".sln");
			FPaths::NormalizeFilename(CachedSolutionPath);
		}
	}

	// This must be an absolute path as VS always uses absolute paths
	return CachedSolutionPath;
}

void FVisualStudioSourceCodeAccessor::Tick(const float DeltaTime)
{
	TArray<FileOpenRequest> TmpDeferredRequests;
	{
		FScopeLock Lock(&DeferredRequestsCriticalSection);

		// Copy the DeferredRequests array, as OpenVisualStudioFilesInternal may update it
		TmpDeferredRequests = DeferredRequests;
		DeferredRequests.Empty();
	}

	if(TmpDeferredRequests.Num() > 0)
	{
		// Try and open any pending files in VS first (this will update the VS launch state appropriately)
		OpenVisualStudioFilesInternal(TmpDeferredRequests);
	}
}

FName FVisualStudioSourceCodeAccessor::GetOpenIconName() const
{
	return FName("MainFrame.OpenVisualStudio");
}

FName FVisualStudioSourceCodeAccessor::GetRefreshIconName() const
{
	return FName("MainFrame.RefreshVisualStudio");
}

#undef LOCTEXT_NAMESPACE
