// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ISourceCodeAccessor.h"
#include "VisualStudioDTE.h"

class FVisualStudioSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	/** Struct representing identifying information about Visual Studio versions */
	struct VisualStudioLocation
	{
		int32 VersionNumber;
		bool bPreviewRelease;
		FString ExecutablePath;
#if WITH_VISUALSTUDIO_DTE
		FString ROTMoniker;
#endif
	};

	FVisualStudioSourceCodeAccessor()
	{
	}

	/** Initialise internal systems, register delegates etc. */
	void Startup();

	/** Shut down internal systems, unregister delegates etc. */
	void Shutdown();

	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override;
	virtual bool CanAccessSourceCode() const override;
	virtual FName GetFName() const override;
	virtual FText GetNameText() const override;
	virtual FText GetDescriptionText() const override;
	virtual bool OpenSolution() override;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;
	virtual bool DoesSolutionExist() const override;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;
	virtual bool SaveAllOpenDocuments() const override;
	virtual void Tick(const float DeltaTime) override;
	virtual FName GetOpenIconName() const override;
	virtual FName GetRefreshIconName() const override;

private:

	struct FileOpenRequest
	{
		FString FullPath;
		int32 LineNumber;
		int32 ColumnNumber;

		FileOpenRequest()
			: LineNumber(0)
			, ColumnNumber(0)
		{
		}

		FileOpenRequest(const FString& InitFullPath, int32 InitLineNumber, int32 InitColumnNumber)
			: FullPath(InitFullPath)
			, LineNumber(InitLineNumber)
			, ColumnNumber(InitColumnNumber)
		{
		}
	};

	/** Checks to see if we can run visual studio, also outputs the executable path if it's needed */
	bool CanRunVisualStudio(FString& OutPath, const FString& InSolution) const;

	/** Run an instance of visual studio instance if possible. */
	bool RunVisualStudioAndOpenSolution(const FString& InSolution) const;

	/** Opens a file in the correct running instance of Visual Studio at a line and optionally to a column. */
	bool OpenVisualStudioFileAtLineInternal(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0);

	/** Opens multiple files in the correct running instance of Visual Studio. */
	bool OpenVisualStudioFilesInternal(const TArray<FileOpenRequest>& Requests);
		
	/** An instance of VS it attempting to be opened */
	void VSLaunchStarted();

	/** 
	 * An instance of VS has finished processing 
	 * 
	 * @param	bSuccess	Whether VS has finished successfully or not
	 */
	void VSLaunchFinished( bool bSuccess );

	/** 
	 * Are we trying to launch an instance of VS
	 * 
	 * @return	true if we're in the middle of a VS instance being launched
	 */
	bool IsVSLaunchInProgress() const { return( ( VSLaunchTime != 0.0) ? true : false ); }

	/** 
	 * Add a new version of Visual Studio to the supported locations array
	 * 
	 * @param	MajorVersion	The major version number of Visual Studio (eg, 17 for VS2022)
	 * @param	bAllowExpress	true to also add the express version of Visual Studio to the list of locations
	 */
	void AddVisualStudioVersion(const int MajorVersion, const bool bAllowExpress = true);

	/**
	 * Add a new version of Visual Studio to the supported locations array using the Visual Studio Setup API
	 *
	 * @param	MinimumVersionNumber	The minimum major version number of Visual Studio (eg, 17 for VS2022) to return
	 */
	void AddVisualStudioVersionUsingVisualStudioSetupAPI(const int MinimumVersionNumber);

	/**
	 * Get the sort weight of a Visual Studio install when ordering by priority. 
	 * 
	 * @param	InLocation		The Visual Studio install
	 * @param	ExactVersion	The specific major version number of Visual Studio to prioritize
	 * @param	bPreferPreview	true if preview releases should be prioritized
	 *
	 * @return	The sort weight of an install location
	 */
	int32 VisualStudioVersionSortWeight(const VisualStudioLocation& InLocation, const int32 ExactVersion, const bool bPreferPreview) const;

	/** 
	 * Get the prioritized list of VS install locations based upon the version of the given solution
	 * 
	 * @return	The sorted array of install locations
	 */
	TArray<VisualStudioLocation> GetPrioritizedVisualStudioVersions(const FString& InSolution) const;

	/** 
	 * Run a new instance Visual Studio, optionally opening the provided solution and list of files
	 * 
	 * @param	ExecutablePath	Path to the Visual Studio executable to open
	 * @param	SolutionPath	Path to the solution to open, or an empty string to open no solution
	 * @param	Requests		Array of files to open, or null to open no files
	 */
	bool RunVisualStudioAndOpenSolutionAndFiles(const FString& ExecutablePath, const FString& SolutionPath, const TArray<FileOpenRequest>* const Requests) const;

	/**
	 * Based on the list of file open requests and the list of solutions currently opened, make an educated guess as to which solution might be the most relevant to open them all
	 *
	 * @param	Requests					Array of files to open
	 * @param	CurrentlyOpenedSolutions	List of solutions currently opened by running VS instances
	 * @return the absolute file path to the most likely solution or an empty string if no solution was found 
	 */
	FString RetrieveSolutionForFileOpenRequests(const TArray<FileOpenRequest>& Requests, const TArray<FString>& CurrentlyOpenedSolutions) const;

#if WITH_VISUALSTUDIO_DTE
	/** DTE specific implementations */
	bool OpenVisualStudioSolutionViaDTE();
	bool OpenVisualStudioFilesInternalViaDTE(const TArray<FileOpenRequest>& Requests, bool& bWasDeferred);
#endif
	/** Fallback (non-DTE) implementations */
	bool OpenVisualStudioSolutionViaProcess();
	bool OpenVisualStudioFilesInternalViaProcess(const TArray<FileOpenRequest>& Requests);

private:

	/** The minimum major version of Visual Studio we support (eg, 17 for Visual Studio 2022) */
	static constexpr int32 MinimumVisualStudioVersion = 17;

	/** The versions of VS we support, in preference order */
	TArray<VisualStudioLocation> Locations;

	/** String storing the solution path obtained from the module manager to avoid having to use it on a thread */
	mutable FString CachedSolutionPath;

	/** Override for the cached solution path */
	mutable FString CachedSolutionPathOverride;

	/** If !0 it represents the time at which the a VS instance was opened */
	double VSLaunchTime;

	/** Critical section for updating DeferredRequests */
	FCriticalSection DeferredRequestsCriticalSection;

	/** Critical section for updating SolutionPath */
	mutable FCriticalSection CachedSolutionPathCriticalSection;

	/** Array of file open requests that have been deferred because Visual Studio was not available when the request was made */
	TArray<FileOpenRequest> DeferredRequests;

	/** Accessor for SolutionPath. Will try to update it when called from the game thread, otherwise will use the cached value */
	FString GetSolutionPath() const;

#if WITH_EDITOR
	FDelegateHandle SaveVisualStudioDocumentsDelegateHandle;
#endif
};
