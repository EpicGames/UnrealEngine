// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualStudioSourceCodeAccessorWrapper.h"
#include "VisualStudioSourceCodeAccessModule.h"
#include "ISourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "DesktopPlatformModule.h"

FVisualStudioSourceCodeAccessorWrapper::FVisualStudioSourceCodeAccessorWrapper(const FName& InName, const FText& InNameText, const FText& InDescriptionText, const TSharedRef<FVisualStudioSourceCodeAccessor>& InInner)
	: Name(InName)
	, NameText(InNameText)
	, DescriptionText(InDescriptionText)
	, Inner(InInner)
{
}

void FVisualStudioSourceCodeAccessorWrapper::RefreshAvailability()
{
	// Inner will get this call separately
}

bool FVisualStudioSourceCodeAccessorWrapper::CanAccessSourceCode() const
{
	return Inner->CanAccessSourceCode();
}

FName FVisualStudioSourceCodeAccessorWrapper::GetFName() const
{
	return Name;
}

FText FVisualStudioSourceCodeAccessorWrapper::GetNameText() const
{
	return NameText;
}

FText FVisualStudioSourceCodeAccessorWrapper::GetDescriptionText() const
{
	return DescriptionText;
}

bool FVisualStudioSourceCodeAccessorWrapper::OpenSolution()
{
	return Inner->OpenSolution();
}

FName FVisualStudioSourceCodeAccessorWrapper::GetOpenIconName() const
{
	return FName("MainFrame.OpenVisualStudio");
}

FName FVisualStudioSourceCodeAccessorWrapper::GetRefreshIconName() const
{
	return FName("MainFrame.RefreshVisualStudio");
}

bool FVisualStudioSourceCodeAccessorWrapper::OpenSolutionAtPath(const FString& InSolutionPath)
{
	return Inner->OpenSolutionAtPath(InSolutionPath);
}

bool FVisualStudioSourceCodeAccessorWrapper::DoesSolutionExist() const
{
	return Inner->DoesSolutionExist();
}

bool FVisualStudioSourceCodeAccessorWrapper::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	return Inner->OpenFileAtLine(FullPath, LineNumber, ColumnNumber);
}

bool FVisualStudioSourceCodeAccessorWrapper::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	return Inner->OpenSourceFiles(AbsoluteSourcePaths);
}

bool FVisualStudioSourceCodeAccessorWrapper::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	return Inner->AddSourceFiles(AbsoluteSourcePaths, AvailableModules);
}

bool FVisualStudioSourceCodeAccessorWrapper::SaveAllOpenDocuments() const
{
	return Inner->SaveAllOpenDocuments();
}

void FVisualStudioSourceCodeAccessorWrapper::Tick(const float DeltaTime)
{
	Inner->Tick(DeltaTime);
}
