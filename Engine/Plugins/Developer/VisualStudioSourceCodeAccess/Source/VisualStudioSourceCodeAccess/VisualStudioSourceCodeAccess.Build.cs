// Copyright Epic Games, Inc. All Rights Reserved.
using Microsoft.Win32;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class VisualStudioSourceCodeAccess : ModuleRules
	{
        public VisualStudioSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.None;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SourceCodeAccess",
					"DesktopPlatform",
					"Projects",
					"Json",
					"VisualStudioSetup",
					"VisualStudioDTE"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("HotReload");
			}

			bBuildLocallyWithSNDBS = true;
			
			ShortName = "VSSCA";
		}
	}
}
