{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Visual Studio Integration",
	"Description" : "Allows access to source code in Visual Studio.",
	"Category" : "Programming",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": [ "UnrealFrontend", "UnrealInsights" ],
	"Modules" :
	[
		{
			"Name" : "VisualStudioSourceCodeAccess",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
			],
			"ProgramAllowList" : [ "UnrealFrontend", "UnrealInsights" ]
		}
	]
}